// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaControllerModule.h"
#include "HttpModule.h"
#include "Misc/ConfigCacheIni.h"
#include "UbaJobProcessor.h"
#include "UbaHordeConfig.h"
#include "Features/IModularFeatures.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "CoreMinimal.h"


DEFINE_LOG_CATEGORY(LogUbaController);

namespace UbaControllerModule
{
	static constexpr int32 SubFolderCount = 32;

	static bool bDumpTraceFiles = true;
	static FAutoConsoleVariableRef CVarDumpTraceFiles(
		TEXT("r.UbaController.DumpTraceFiles"),
		bDumpTraceFiles,
		TEXT("If true, UBA controller dumps trace files for later use with UBA visualizer in the Saved folder under UbaController (Enabled by default)"));

	static FString MakeAndGetDebugInfoPath()
	{
		// Build machines should dump to the AutomationTool/Saved/Logs directory and they will upload as build artifacts via the AutomationTool.
		FString BaseDebugInfoPath = FPaths::ProjectSavedDir();
		if (GIsBuildMachine)
		{
			BaseDebugInfoPath = FPaths::Combine(*FPaths::EngineDir(), TEXT("Programs"), TEXT("AutomationTool"), TEXT("Saved"), TEXT("Logs"));
		}

		FString AbsoluteDebugInfoDirectory = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*(BaseDebugInfoPath / TEXT("UbaController")));
		FPaths::NormalizeDirectoryName(AbsoluteDebugInfoDirectory);

		// Create directory if it doesn't exit yet
		if (!IFileManager::Get().DirectoryExists(*AbsoluteDebugInfoDirectory))
		{
			IFileManager::Get().MakeDirectory(*AbsoluteDebugInfoDirectory, true);
		}

		return AbsoluteDebugInfoDirectory;
	}

	static const FString GetTempDir()
	{
		static FString HordeSharedDir = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_SHARED_DIR"));
		if (HordeSharedDir.IsEmpty())
			HordeSharedDir = FPlatformProcess::UserTempDir();
		return HordeSharedDir;
	}

};

FUbaControllerModule::FUbaControllerModule()
	: bSupported(false)
	, bModuleInitialized(false)
	, bControllerInitialized(false)
	, RootWorkingDirectory(FPaths::Combine(*UbaControllerModule::GetTempDir(), TEXT("UbaControllerWorkingDir")))
	, WorkingDirectory(FPaths::Combine(RootWorkingDirectory, FGuid::NewGuid().ToString(EGuidFormats::Digits)))
	, NextFileID(0)
	, NextTaskID(0)
{
}

FString FUbaControllerModule::GetTempDir()
{
	return UbaControllerModule::GetTempDir();
}

FUbaControllerModule::~FUbaControllerModule()
{	
	StopAndWaitForDispatcherThread();
	CleanWorkingDirectory();
}

static bool IsUbaControllerEnabled()
{
	if (FParse::Param(FCommandLine::Get(), TEXT("NoUbaController")) || // Compatibility to old terminology
		FParse::Param(FCommandLine::Get(), TEXT("NoUbaShaderCompile")) || // Same terminology as other shader controllers
		FParse::Param(FCommandLine::Get(), TEXT("NoShaderWorker")))
	{
		return false;
	}

	// Check if UbaController is enabled via command line argument
	if (FParse::Param(FCommandLine::Get(), TEXT("UBA")) ||
		FParse::Param(FCommandLine::Get(), TEXT("UBAEnableHorde")))
	{
		return true;
	}

	// Check if UbaController is enabled via INI configuration in [UbaController] section.
	return FUbaHordeConfig::Get().bIsProviderEnabled;
}

bool FUbaControllerModule::IsSupported()
{
	if (bControllerInitialized)
	{
		return bSupported;
	}
	
	const bool bEnabled = IsUbaControllerEnabled();

	bSupported = FPlatformProcess::SupportsMultithreading() && bEnabled;
	return bSupported;
}

void FUbaControllerModule::CleanWorkingDirectory() const
{
	if (UE::GetMultiprocessId() != 0) // Only director is allowed to clean
	{
		return;
	}

	IFileManager& FileManager = IFileManager::Get();
	
	if (!RootWorkingDirectory.IsEmpty())
	{
		if (!FileManager.DeleteDirectory(*RootWorkingDirectory, false, true))
		{
			UE_LOGF(LogUbaController, Log, "%ls => Failed to delete current working Directory => %ls", ANSI_TO_TCHAR(__FUNCTION__), *RootWorkingDirectory);
		}	
	}
}

FString GetUbaBinariesPath()
{
#if PLATFORM_WINDOWS
#if PLATFORM_CPU_ARM_FAMILY
	const TCHAR* BinariesArch = TEXT("arm64");
#else
	const TCHAR* BinariesArch = TEXT("x64");
#endif
	return FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries"), TEXT("Win64"), TEXT("UnrealBuildAccelerator"), BinariesArch);
#elif PLATFORM_MAC
    return FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries"), TEXT("Mac"), TEXT("UnrealBuildAccelerator"));
#elif PLATFORM_LINUX
    return FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries"), TEXT("Linux"), TEXT("UnrealBuildAccelerator"));
#else
#error Unsupported platform to compile UbaController plugin. Only Win64, Mac, and Linux are supported!
#endif
}

void FUbaControllerModule::LoadDependencies()
{
	const FString UbaBinariesPath = GetUbaBinariesPath();
	FPlatformProcess::AddDllDirectory(*UbaBinariesPath);
	#if PLATFORM_WINDOWS
	FPlatformProcess::GetDllHandle(*(FPaths::Combine(UbaBinariesPath, "UbaHost.dll")));
	#elif PLATFORM_LINUX
	FPlatformProcess::GetDllHandle(*(FPaths::Combine(UbaBinariesPath, "libUbaHost.so")));
	#endif
}

void FUbaControllerModule::StartupModule()
{
	check(!bModuleInitialized);

	LoadDependencies();

	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureType(), this);

	bModuleInitialized = true;

	FCoreDelegates::OnEnginePreExit.AddLambda([&]()
	{
		if (bControllerInitialized)
		{
			StopAndWaitForDispatcherThread();
		}
	});
}

void FUbaControllerModule::ShutdownModule()
{
	check(bModuleInitialized);
	
	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureType(), this);
	
	if (bControllerInitialized)
	{
		// Stop the jobs thread
		StopAndWaitForDispatcherThread();

		FDistributedBuildTask* Task;
		while (PendingRequestedCompilationTasks.Dequeue(Task))
		{
			Task->Cancel();
			delete Task;
		}

		PendingRequestedCompilationTasks.Empty();
		PendingRequestedCompilationCount = 0;
	}

	CleanWorkingDirectory();
	bModuleInitialized = false;
	bControllerInitialized = false;
}

void FUbaControllerModule::InitializeController()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUbaControllerModule::InitializeController);

	// We should never Initialize the controller twice
	if (ensureAlwaysMsgf(!bControllerInitialized, TEXT("Multiple initialization of UBA controller!")))
	{
		CleanWorkingDirectory();

		if (IsSupported())
		{
			IFileManager::Get().MakeDirectory(*WorkingDirectory, true);

			// Pre-create the directories so we don't have to explicitly register them to uba later
			for (int32 FolderIndex = 0; FolderIndex < UbaControllerModule::SubFolderCount; ++FolderIndex)
			{
				IFileManager::Get().MakeDirectory(*FPaths::Combine(WorkingDirectory, FString::FromInt(FolderIndex)));
			}

			if (UbaControllerModule::bDumpTraceFiles)
			{
				DebugInfoPath = UbaControllerModule::MakeAndGetDebugInfoPath();
			}

            // Make sure HTTP and Scokets modules are loaded in game thread before launching UBA client
			FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
			FModuleManager::LoadModuleChecked<FHttpModule>("Sockets");

			StartDispatcherThread();
		}

		bControllerInitialized = true;
	}
}

bool FUbaControllerModule::SupportsLocalWorkers()
{
	// UbaController supports local workers if the maximum number of local cores is greater than zero or -1 (special value to use all available cores)
	const int32 MaxLocalCores = FUbaHordeConfig::Get().MaxParallelActions;
	return MaxLocalCores > 0 || MaxLocalCores == -1;
}

FString FUbaControllerModule::CreateUniqueFilePath()
{
	check(bSupported);
	const int32 FileID = NextFileID++;
	const int32 FolderID = FileID % UbaControllerModule::SubFolderCount; // We use sub folders to be nicer to file system (we can end up with 20000 files in one folder otherwise)
	return FPaths::Combine(WorkingDirectory, FString::FromInt(FolderID), FString::Printf(TEXT("%d.uba"), FileID));
}

bool FUbaControllerModule::SupportsVirtualFiles() const
{
	return JobDispatcherThread != nullptr && JobDispatcherThread->IsUsingVirtualFiles();
}

TFuture<FDistributedBuildTaskResult> FUbaControllerModule::EnqueueTask(const FTaskCommandData& CommandData)
{
	check(bSupported);

	TPromise<FDistributedBuildTaskResult> Promise;
	TFuture<FDistributedBuildTaskResult> Future = Promise.GetFuture();

	// Enqueue the new task
	FDistributedBuildTask* Task = new FDistributedBuildTask(NextTaskID++, CommandData, MoveTemp(Promise));
	{
		PendingRequestedCompilationTasks.Enqueue(Task);
		PendingRequestedCompilationCount++;
		ProcessorWakeEvent->Trigger();
	}

	return MoveTemp(Future);
}

bool FUbaControllerModule::PollStats(FDistributedBuildStats& OutStats)
{
	return JobDispatcherThread != nullptr && JobDispatcherThread->PollStats(OutStats);
}

UBACONTROLLER_API FUbaControllerModule& FUbaControllerModule::Get()
{
	return FModuleManager::LoadModuleChecked<FUbaControllerModule>(TEXT("UbaController"));
}

void FUbaControllerModule::StartDispatcherThread()
{
	checkf(JobDispatcherThread == nullptr, TEXT("UBA job dispatcher thread was not previoulsy shut down before starting it again"));
	JobDispatcherThread = MakeUnique<FUbaJobProcessor>(*this);
	JobDispatcherThread->StartThread();
}

void FUbaControllerModule::StopAndWaitForDispatcherThread()
{
	if (JobDispatcherThread)
	{
		JobDispatcherThread->Stop();
		ProcessorWakeEvent->Trigger();
		// Wait until the thread is done
		FPlatformProcess::ConditionalSleep([&]() { return !JobDispatcherThread || JobDispatcherThread->IsWorkDone(); }, 0.1f);
		JobDispatcherThread = nullptr;
	}
}

IMPLEMENT_MODULE(FUbaControllerModule, UbaController);
