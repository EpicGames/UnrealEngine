// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaJobProcessor.h"

#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreMisc.h"
#include "Misc/FileHelper.h"
#include "Misc/PathViews.h"
#include "Serialization/MemoryReader.h"
#include "UbaControllerModule.h"
#include "UbaHordeAgentManager.h"
#include "UbaHordeConfig.h"
#include "UbaStringConversion.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"

#include "Misc/Paths.h"
#include "Misc/ScopeRWLock.h"

namespace UbaJobProcessorOptions
{
	static float MaxTimeWithoutTasks = 100.0f;
	static FAutoConsoleVariableRef CVarMaxTimeWithoutTasks(
		TEXT("r.UbaController.MaxTimeWithoutTasks"),
		MaxTimeWithoutTasks,
		TEXT("Time to wait for pending tasks (in seconds) when the queue is empty before shutting down the UBA/Horde session.\n"));

	static float HeartBeatInterval = 180.0f;
	static FAutoConsoleVariableRef CVarHeartBeatInterval(
		TEXT("r.UbaController.HeartBeatInterval"),
		HeartBeatInterval,
		TEXT("Time between heart beat log messages"));

	static float ProcessIdleTime = 15.0f;
	static FAutoConsoleVariableRef CVarProcessIdleTime(
		TEXT("r.UbaController.ProcessIdleTime"),
		ProcessIdleTime,
		TEXT("Time ShaderCompileWorker will wait for new tasks before exiting. Only used when process reuse is enabled. Set to 0 to only re-use processes if there are tasks in the queue pending assignment."));

	static bool bAutoLaunchVisualizer = false;
	static FAutoConsoleVariableRef CVarAutoLaunchVisualizer(
		TEXT("r.UbaController.AutoLaunchVisualizer"),
		bAutoLaunchVisualizer,
		TEXT("If true, UBA visualizer will be launched automatically\n"));

	static bool bUseTraceChannel = false;
	static FAutoConsoleVariableRef CVarUseTraceChannel(
		TEXT("r.UbaController.UseTraceChannel"),
		bUseTraceChannel,
		TEXT("If true, UBA sessions announce their trace on a per-MultiprocessId named channel (UbaController / UbaController_Worker<N>) and a single visualizer is reused across subsequent sessions using the same channel. If false, traces post to the default channel and each session spawns its own visualizer.\n"));

	static bool bAllowProcessReuse = true;
	static FAutoConsoleVariableRef CVarAllowProcessReuse(
		TEXT("r.UbaController.AllowProcessReuse"),
		bAllowProcessReuse,
		TEXT("If true, remote process is allowed to fetch new processes from the queue (this requires the remote processes to have UbaRequestNextProcess implemented)\n"));

	static bool bDetailedTrace = false;
	static FAutoConsoleVariableRef CVarDetailedTrace(
		TEXT("r.UbaController.DetailedTrace"),
		bDetailedTrace,
		TEXT("If true, a UBA will output detailed trace\n"));

	enum EUbaLogVerbosity
	{
		UbaLogVerbosity_Default = 0, // foward erros and warnings only
		UbaLogVerbosity_High, // also forward infos
		UbaLogVerbosity_Max // forward all UBA logs to UE_LOG
	};

	static int32 UbaLogVerbosity = UbaLogVerbosity_Default;
	static FAutoConsoleVariableRef CVarShowUbaLog(
		TEXT("r.UbaController.LogVerbosity"),
		UbaLogVerbosity,
		TEXT("Specifies how much of UBA logs is forwarded to UE logs..\n")
		TEXT("0 - Default, only forward errrors and warnings.\n")
		TEXT("1 - Also forward regular information about UBA sessions.\n")
		TEXT("2 - Forward all UBA logs."));

	static int32 SaveUbaTraceSnapshotInterval = 0;
	static FAutoConsoleVariableRef CVarSaveUbaTraceSnapshotInterval(
		TEXT("r.UbaController.SaveTraceSnapshotInterval"),
		SaveUbaTraceSnapshotInterval,
		TEXT("Specifies the interval (in seconds) in which a snapshot of the current state of the UBA trace will be saved to file.\n")
		TEXT("A value of 0 disables the periodic snapshots and only saves the UBA trace at the end of each server session. By default 0.\n"));

	static bool bProcessLogEnabled = false;
	static FAutoConsoleVariableRef CVarProcessLogEnabled(
		TEXT("r.UbaController.ProcessLogEnabled"),
		bProcessLogEnabled,
		TEXT("If true, each detoured process will write a log file. Note this is only useful if UBA is compiled in debug\n"));

#if PLATFORM_WINDOWS
	static bool bUseVirtualFiles = true;
#else
	// virtual file mechanism (which relies on shared memory) is not currently implemented in UBA for Mac/Linux, so we cannot use this on those platforms until that is fixed
	static bool bUseVirtualFiles = false;
#endif
	static FAutoConsoleVariableRef CVarUseVirtualFiles(
		TEXT("r.UbaController.UseVirtualFiles"),
		bUseVirtualFiles,
		TEXT("If true, UBA virtual file mechanism will be used for input and output files (avoiding writing them to disk)"));

	FString ReplaceEnvironmentVariablesInPath(const FString& ExtraFilePartialPath) // Duplicated code with FAST build.. put it somewhere else?
	{
		FString ParsedPath;

		// Fast build cannot read environmental variables easily
		// Is better to resolve them here
		if (ExtraFilePartialPath.Contains(TEXT("%")))
		{
			TArray<FString> PathSections;
			ExtraFilePartialPath.ParseIntoArray(PathSections, TEXT("/"));

			for (FString& Section : PathSections)
			{
				if (Section.Contains(TEXT("%")))
				{
					Section.RemoveFromStart(TEXT("%"));
					Section.RemoveFromEnd(TEXT("%"));
					Section = FPlatformMisc::GetEnvironmentVariable(*Section);
				}
			}

			for (FString& Section : PathSections)
			{
				ParsedPath /= Section;
			}

			FPaths::NormalizeDirectoryName(ParsedPath);
		}

		if (ParsedPath.IsEmpty())
		{
			ParsedPath = ExtraFilePartialPath;
		}

		return ParsedPath;
	}
}

FUbaJobProcessor::FUbaJobProcessor(
	FUbaControllerModule& InControllerModule) :

	Thread(nullptr),
	ControllerModule(InControllerModule),
	bForceStop(false),
	LastTimeCheckedForTasks(0),
	bIsWorkDone(false),
	LogWriter([]() {}, []() {}, [](uba::LogEntryType type, const uba::tchar* str, uba::u32 /*strlen*/)
	{
			switch (type)
			{
			case uba::LogEntryType_Error:
				UE_LOGF(LogUbaController, Error, "%ls", UBASTRING_TO_TCHAR(str));
				break;
			case uba::LogEntryType_Warning:
				UE_LOGF(LogUbaController, Warning, "%ls", UBASTRING_TO_TCHAR(str));
				break;
			case uba::LogEntryType_Info:
				if (UbaJobProcessorOptions::UbaLogVerbosity >= UbaJobProcessorOptions::UbaLogVerbosity_High)
				{
					UE_LOGF(LogUbaController, Display, "%ls", UBASTRING_TO_TCHAR(str));
				}
				break;
			default:
				if (UbaJobProcessorOptions::UbaLogVerbosity >= UbaJobProcessorOptions::UbaLogVerbosity_Max)
				{
					UE_LOGF(LogUbaController, Display, "%ls", UBASTRING_TO_TCHAR(str));
				}
				break;
			}
	})
{
	Uba_SetCustomAssertHandler([](const uba::tchar* text)
		{
			checkf(false, TEXT("%s"), UBASTRING_TO_TCHAR(text));
		});

	UpdateMaxLocalParallelJobs();
}

FUbaJobProcessor::~FUbaJobProcessor()
{
	delete Thread;
}

void FUbaJobProcessor::UpdateMaxLocalParallelJobs()
{
	// Limit number of parallel jobs by UBA/Horde configuration
	MaxLocalParallelJobs = FUbaHordeConfig::Get().MaxParallelActions;
	if (MaxLocalParallelJobs == -1)
	{
		MaxLocalParallelJobs = FPlatformMisc::NumberOfCoresIncludingHyperthreads();
	}

	// Also apply limits from shader compiling manager, i.e. [DevOptions.Shaders]:NumUnusedShaderCompilingThreads etc.
	const int32 MaxNumLocalWorkers = ControllerModule.GetMaxNumLocalWorkers();
	if (MaxNumLocalWorkers != -1)
	{
		MaxLocalParallelJobs = FMath::Min(MaxLocalParallelJobs, MaxNumLocalWorkers);
	}
}

void FUbaJobProcessor::CalculateKnownInputs()
{
	// TODO: This is ShaderCompileWorker specific and this code is designed to handle all kinds of distributed workload.
	// Instead this information should be provided from the outside


	if (KnownInputsCount) // In order to improve startup we provide some of the input we know will be loaded by ShaderCompileWorker.exe
	{
		return;
	}

	auto AddKnownInput = [&](const FString& file)
		{
			#if PLATFORM_WINDOWS
			auto& fileData = file.GetCharArray();
			const uba::tchar* fileName = fileData.GetData();
			size_t fileNameLen = fileData.Num();
			#else
			FStringToUbaStringConversion conv(*file);
			const uba::tchar* fileName = conv.Get();
			size_t fileNameLen = strlen(fileName) + 1;
			#endif
			auto num = KnownInputsBuffer.Num();
			KnownInputsBuffer.SetNum(num + fileNameLen);
			memcpy(KnownInputsBuffer.GetData() + num, fileName, fileNameLen * sizeof(uba::tchar));
			++KnownInputsCount;
		};

	// Get the binaries
	TArray<FString> KnownFileNames;
	FString BinDir = FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries"), FPlatformProcess::GetBinariesSubdirectory());

	#if PLATFORM_WINDOWS
	AddKnownInput(*FPaths::Combine(BinDir, TEXT("ShaderCompileWorker.exe")));
	#else
	AddKnownInput(*FPaths::Combine(BinDir, TEXT("ShaderCompileWorker")));
	#endif

	IFileManager::Get().FindFilesRecursive(KnownFileNames, *BinDir, TEXT("ShaderCompileWorker-*.*"), true, false);
	for (const FString& file : KnownFileNames)
	{
		if (file.EndsWith(FPlatformProcess::GetModuleExtension()))
		{
			AddKnownInput(file);
		}
	}

	// Get the compiler dependencies for all platforms)
	if (ITargetPlatformManagerModule* TargetPlatformManager = GetTargetPlatformManager())
	{
		for (ITargetPlatform* TargetPlatform : TargetPlatformManager->GetTargetPlatforms())
		{
			KnownFileNames.Empty();
			TargetPlatform->GetShaderCompilerDependencies(KnownFileNames);

			for (const FString& ExtraFilePartialPath : KnownFileNames)
			{
				if (!ExtraFilePartialPath.Contains(TEXT("*"))) // Seems like there are some *.x paths in there.. TODO: Do a find files
				{
					AddKnownInput(UbaJobProcessorOptions::ReplaceEnvironmentVariablesInPath(ExtraFilePartialPath));
				}
			}
		}
	}

	// Get all the config files
	for (const FString& ConfigDir : FPaths::GetExtensionDirs(FPaths::EngineDir(), TEXT("Config")))
	{
		KnownFileNames.Empty();
		IFileManager::Get().FindFilesRecursive(KnownFileNames, *ConfigDir, TEXT("*.ini"), true, false);
		for (const FString& file : KnownFileNames)
		{
			AddKnownInput(file);
		}
	}

	KnownInputsBuffer.Add(0);
}

void FUbaJobProcessor::RunTaskWithUba(FDistributedBuildTask* Task)
{
	FTaskCommandData& Data = Task->CommandData;

	FString InputFileName = FPaths::GetCleanFilename(Data.InputFileName);
	FString AppDir = FPaths::GetPath(Data.Command);
	FStringToUbaStringConversion UbaInputFileNameStr(*InputFileName);
	const float TimeToLive = UbaJobProcessorOptions::bAllowProcessReuse ? UbaJobProcessorOptions::ProcessIdleTime : 0.0f;
	// UBA appends its own process-reuse flag. Workers that don't care about -TimeToLive should ignore it.
	const FString Parameters = FString::Printf(TEXT("%s -TimeToLive=%.1f"), *Data.ExtraCommandArgs, TimeToLive);

	uba::Config* Config = Config_Create();
	uba::ConfigTable* RootTable = Config_RootTable(*Config);
	ConfigTable_AddValueString(*RootTable, TC("Application"), FStringToUbaStringConversion(*Data.Command).Get());
	ConfigTable_AddValueString(*RootTable, TC("Arguments"), FStringToUbaStringConversion(*Parameters).Get());
	ConfigTable_AddValueString(*RootTable, TC("Description"), UbaInputFileNameStr.Get());
	ConfigTable_AddValueString(*RootTable, TC("WorkingDir"), FStringToUbaStringConversion(*AppDir).Get());
	ConfigTable_AddValueString(*RootTable, TC("Breadcrumbs"), FStringToUbaStringConversion(*Data.Description).Get());
	ConfigTable_AddValueBool(*RootTable,   TC("WriteOutputFilesOnFail"), true);

	if (UbaJobProcessorOptions::bProcessLogEnabled)
	{
		ConfigTable_AddValueString(*RootTable, TC("LogFile"), UbaInputFileNameStr.Get());
	}

	uint32 SourceVirtualFileCount = 0;
	if (UbaJobProcessorOptions::bUseVirtualFiles)
	{
		// create a map of "overlay" files for the process; doing so prevents dir/file table bloat by ensuring these temp files don't get stored in these tables indefinitely, only for the lifetime of the process
		// note that this is only done if we're using virtual files since it requires shared memory to be enabled (and we keep this off when using the file-based path)
		TStringBuilder<512> OverlayFiles;
		OverlayFiles << Data.InputFileName;

		// register the main input file as a virtual file
		if (!SessionServer_CreateVirtualFile(UbaSessionServer, TCHAR_TO_UBASTRING(*Data.InputFileName), Data.InputFileData.GetData(), (uint64)Data.InputFileData.NumBytes(), true, false))
		{
			UE_LOGF(LogUbaController, Warning, "SessionServer_CreateVirtualFile failed (%ls)", *Data.InputFileName);
		}

		// register each source file required for the batch of tasks as an individual virtual file, using the same base filename as the input file with a sourcefile index appended
		TStringBuilder<512> SourceVirtualFileName;
		for (const FMemoryView& Source : Data.SourceData)
		{
			SourceVirtualFileName.Reset();
			SourceVirtualFileName << Data.InputFileName << SourceVirtualFileCount++;

			OverlayFiles << ';' << SourceVirtualFileName;

			if (!SessionServer_CreateVirtualFile(UbaSessionServer, TCHAR_TO_UBASTRING(SourceVirtualFileName.ToString()), Source.GetData(), Source.GetSize(), true, false))
			{
				UE_LOGF(LogUbaController, Warning, "SessionServer_CreateVirtualFile failed (%ls)", *SourceVirtualFileName);
			}
		}

		ConfigTable_AddValueString(*RootTable, TC("OverlayFiles"), FStringToUbaStringConversion(*OverlayFiles).Get());
	}
	else
	{
		SessionServer_RegisterNewFile(UbaSessionServer, TCHAR_TO_UBASTRING(*Data.InputFileName));
	}

	for (const FString& AdditionalOutputFolder : Data.AdditionalOutputFolders)
	{
		SessionServer_RegisterNewDirectory(UbaSessionServer, TCHAR_TO_UBASTRING(*AdditionalOutputFolder));
	}

	struct FUbaProcessData
	{
		FUbaJobProcessor* Processor;
		FString InputFile;
		FString OutputFile;
		FDistributedBuildTask* Task;
		uint32 SourceVirtualFileCount;
	};

	auto ProcessData = new FUbaProcessData;
	ProcessData->Processor = this;
	ProcessData->InputFile = Data.InputFileName;
	ProcessData->OutputFile = Data.OutputFileName;
	ProcessData->SourceVirtualFileCount = SourceVirtualFileCount;
	ProcessData->Task = Task;

	static auto ExitedFunc = [](void* UserData, const uba::ProcessHandle& Process)
		{
			for (uint32 LogLineIndex = 0; const uba::tchar* LogLine = ProcessHandle_GetLogLine(&Process, LogLineIndex); ++LogLineIndex)
			{
				UE_LOGF(LogUbaController, Display, "%ls", UBASTRING_TO_TCHAR(LogLine));
			}

			if (FUbaProcessData* ProcessData = static_cast<FUbaProcessData*>(UserData)) // It can be null if custom message has already handled all of them
			{
				struct FProcessOutputData
				{
					FUbaProcessData* ProcessData;
					TArray<uint8> OutputFileBytes;
				};
				FProcessOutputData OutputData{ ProcessData };

				if (UbaJobProcessorOptions::bUseVirtualFiles)
				{
					SessionServer_DeleteVirtualFile(ProcessData->Processor->UbaSessionServer, TCHAR_TO_UBASTRING(*ProcessData->InputFile), false);

					TStringBuilder<512> SourceVirtualFileName;
					for (uint32 SourceIndex = 0; SourceIndex < ProcessData->SourceVirtualFileCount; ++SourceIndex)
					{
						SourceVirtualFileName.Reset();
						SourceVirtualFileName << ProcessData->InputFile << SourceIndex;

						SessionServer_DeleteVirtualFile(ProcessData->Processor->UbaSessionServer, TCHAR_TO_UBASTRING(SourceVirtualFileName.ToString()), false);
					}

					// since we set ShouldWriteToDisk to false in the configuration, we need to manually process the output files from the job batch
					// the main output file never needs to be written to disk; we process the data directly in-memory. all other outputs are debug
					// artifacts, so those should be written out to disk if they exist in the process data
					auto OutputFileCallback = [](void* InUserData, const uba::tchar* OutputFileName)
						{
							FProcessOutputData* UserData = static_cast<FProcessOutputData*>(InUserData);
							FUbaProcessData* ProcessData = UserData->ProcessData;
							FString OutputFileNameStr = UBASTRING_TO_TCHAR(OutputFileName);
							FStringView BaseOutputFileName = FPathViews::GetCleanFilename(ProcessData->OutputFile);
							if (OutputFileNameStr.EndsWith(BaseOutputFileName))
							{
								uint64 OutputFileSize = 0;
								SessionServer_GetOutputFileSize(ProcessData->Processor->UbaSessionServer, OutputFileSize, OutputFileName);
								checkf(OutputFileSize < INT_MAX, TEXT("SCW output file size too large for a 32-bit indexed TArray (%llu)"), OutputFileSize);
								UserData->OutputFileBytes.SetNumUninitialized(OutputFileSize);

								SessionServer_GetOutputFileData(ProcessData->Processor->UbaSessionServer, UserData->OutputFileBytes.GetData(), OutputFileName, true);
							}
							else
							{
								SessionServer_WriteOutputFile(ProcessData->Processor->UbaSessionServer, OutputFileName, true);
							}
						};
					ProcessHandle_TraverseOutputFiles(const_cast<uba::ProcessHandle*>(&Process), OutputFileCallback, &OutputData);
				}
				else
				{
					IFileManager::Get().Delete(*ProcessData->InputFile);
					FStringToUbaStringConversion InputFileUba(*ProcessData->InputFile);
					SessionServer_RegisterDeleteFile(ProcessData->Processor->UbaSessionServer, InputFileUba.Get());
					StorageServer_DeleteFile(ProcessData->Processor->UbaStorageServer, InputFileUba.Get());
					
					FFileHelper::LoadFileToArray(OutputData.OutputFileBytes, *ProcessData->OutputFile);
					StorageServer_DeleteFile(ProcessData->Processor->UbaStorageServer, TCHAR_TO_UBASTRING(*ProcessData->OutputFile));

				}
				ProcessData->Processor->HandleUbaJobFinished(ProcessData->Task, MoveTemp(OutputData.OutputFileBytes), ProcessHandle_GetExitCode(&Process));
				delete ProcessData->Task;
				delete ProcessData;
			}
		};


	uba::ProcessStartInfo* StartInfo = ProcessStartInfo_Create3(*Config);
	ProcessStartInfo_SetExitedCallback(*StartInfo, ExitedFunc, ProcessData);

	Scheduler_EnqueueProcess(UbaScheduler, *StartInfo, 1.0f, KnownInputsBuffer.GetData(), KnownInputsBuffer.Num()*sizeof(uba::tchar), KnownInputsCount);

	ProcessStartInfo_Destroy(StartInfo);
	Config_Destroy(Config);
}

FString GetUbaBinariesPath();

void FUbaJobProcessor::StartUba()
{
	static uint32 UbaSessionCounter = 0;
	const uint32 ThisSession = UbaSessionCounter++;
	const uint32 MultiprocessId = UE::GetMultiprocessId();
	const uint32 EditorPid = FPlatformProcess::GetCurrentProcessId();
	const FString TraceChannelName = (MultiprocessId == 0)
		? FString(TEXT("UbaController"))
		: FString::Printf(TEXT("UbaController_Worker%u"), MultiprocessId);
	// The PID disambiguates trace names across editor restarts so a visualizer that
	// retains a handle to an old trace mapping doesn't collide with the new session.
	FString TraceName = FString::Printf(TEXT("%s_%u_%u"), *TraceChannelName, EditorPid, ThisSession);
	UE_LOGF(LogUbaController, Display, "Starting up UBA/Horde connection for session %ls", *TraceName);

	checkf(UbaServer == nullptr, TEXT("FUbaJobProcessor::StartUba() was called twice before FUbaJobProcessor::ShutDownUba()"));

	FString RootDir;
	int32 FolderIndex = 0;
	while (true)
	{
		RootDir = FPaths::Combine(*FUbaControllerModule::GetTempDir(), TEXT("UbaControllerStorageDir"), FString::FromInt(FolderIndex));
		if (Uba_GetExclusiveAccess(TCHAR_TO_UBASTRING(*RootDir)))
		{
			break;
		}
		++FolderIndex;
	}
	IFileManager::Get().MakeDirectory(*RootDir, true);

	if (!ControllerModule.GetDebugInfoPath().IsEmpty())
	{
		TraceOutputFilename = ControllerModule.GetDebugInfoPath() / FString::Printf(TEXT("UbaController.MultiprocessId-%u.Session-%u.uba"), MultiprocessId, ThisSession);
	}

	uba::Config* Config = Config_Create();
	{
		uba::ConfigTable* RootTable = Config_RootTable(*Config);
		ConfigTable_AddValueString(*RootTable,     TC("RootDir"), TCHAR_TO_UBASTRING(*RootDir));

		uba::ConfigTable* StorageTable = Config_AddTable(*Config, TC("Storage"));
		const uint64 CasSizeMb = UbaJobProcessorOptions::bUseVirtualFiles ? 512 : 32 * 1024; // CAS is basically only populated with SCW and SCW-related binaries when using virtual files, so CAS can be much smaller.
		ConfigTable_AddValueU64(*StorageTable,     TC("CasCapacityBytes"), CasSizeMb * 1024 * 1024);

		uba::ConfigTable* SessionTable = Config_AddTable(*Config, TC("Session"));
		ConfigTable_AddValueBool(*SessionTable,    TC("LaunchVisualizer"), UbaJobProcessorOptions::bAutoLaunchVisualizer);
		ConfigTable_AddValueBool(*SessionTable,    TC("AllowMemoryMaps"), UbaJobProcessorOptions::bUseVirtualFiles);
		ConfigTable_AddValueBool(*SessionTable,    TC("RemoteLogEnabled"), UbaJobProcessorOptions::bProcessLogEnabled);
		ConfigTable_AddValueBool(*SessionTable,    TC("TraceEnabled"), true);
		ConfigTable_AddValueString(*SessionTable,  TC("TraceOutputFile"), TCHAR_TO_UBASTRING(*TraceOutputFilename));
		ConfigTable_AddValueBool(*SessionTable,    TC("DetailedTrace"), UbaJobProcessorOptions::bDetailedTrace);
		ConfigTable_AddValueString(*SessionTable,  TC("TraceName"), TCHAR_TO_UBASTRING(*TraceName));
		if (UbaJobProcessorOptions::bUseTraceChannel)
		{
			ConfigTable_AddValueString(*SessionTable,  TC("TraceChannel"), TCHAR_TO_UBASTRING(*TraceChannelName));
		}
		ConfigTable_AddValueBool(*SessionTable,    TC("ShouldWriteToDisk"), !UbaJobProcessorOptions::bUseVirtualFiles); // files generated by processes should not be written to disk if virtual files are enabled, this will be handled manually
		
		// ConfigTable_AddValueBool(*SessionTable,    TC("RemoteTraceEnabled"), true); Enable this to have the remotes send back the uba trace to the host (ends up in log folder)

		uba::ConfigTable* SchedulerTable = Config_AddTable(*Config, TC("Scheduler"));
		ConfigTable_AddValueU32(*SchedulerTable,   TC("MaxLocalProcessors"), MaxLocalParallelJobs);
		ConfigTable_AddValueBool(*SchedulerTable,  TC("EnableProcessReuse"), UbaJobProcessorOptions::bAllowProcessReuse);
	}


	UbaServer = NetworkServer_Create(LogWriter);
	UbaStorageServer = StorageServer_Create2(*UbaServer, *Config, LogWriter);
	UbaSessionServer = SessionServer_Create2(*UbaStorageServer, *UbaServer, *Config, LogWriter);
	UbaScheduler = Scheduler_Create2(*UbaSessionServer, *Config);

	Config_Destroy(Config);

	// Config used by clients that connect
	{
		uba::Config* ClientConfig = Config_Create();
		uba::ConfigTable* StorageTable = Config_AddTable(*ClientConfig, TC("Storage"));
		ConfigTable_AddValueBool(*StorageTable,  TC("ResendCas"), true); // Since we call StorageServer_DeleteFile there is a tiny risk we might delete a cas file that is needed in the future
		NetworkServer_SetClientsConfig(UbaServer, *ClientConfig);
		Config_Destroy(ClientConfig);
	}

	CalculateKnownInputs();
	UpdateMaxLocalParallelJobs();

	Scheduler_Start(UbaScheduler);

	if (FolderIndex == 0)
	{
		NetworkServer_StartListen(UbaServer, uba::DefaultPort, nullptr); // Start listen so any helper on the LAN can join in
	}

	// Only request Horde agents if Horde is enabled for UBA
	if (FUbaHordeConfig::Get().bIsProviderEnabled)
	{
		SessionServer_UpdateStatus(UbaSessionServer, 0, 1, TC("Horde"), uba::LogEntryType_Info, nullptr);
		SessionServer_UpdateStatus(UbaSessionServer, 0, 6, TC("Starting"), uba::LogEntryType_Info, nullptr);
		HordeAgentManager = MakeUnique<FUbaHordeAgentManager>(ControllerModule.GetWorkingDirectory(), GetUbaBinariesPath());

		HordeAgentManager->SetAddClientCallback([](void* UserData, const uba::tchar* Ip, uint16 Port, const uba::tchar* Crypto16Characters)
			{
				return NetworkServer_AddClient(reinterpret_cast<uba::NetworkServer*>(UserData), Ip, Port, Crypto16Characters);
			}, UbaServer);

		HordeAgentManager->SetUpdateStatusCallback([](void* UserData, const TCHAR* Status)
			{
				SessionServer_UpdateStatus(static_cast<uba::SessionServer*>(UserData), 0, 6, TCHAR_TO_UBASTRING(Status), uba::LogEntryType_Info, nullptr);
			}, UbaSessionServer);
	}

	UE_LOGF(LogUbaController, Display, "Created UBA storage server: RootDir=%ls", *RootDir);
}

void FUbaJobProcessor::ShutDownUba()
{
	UE_LOGF(LogUbaController, Display, "Shutting down UBA/Horde connection");

	HordeAgentManager = nullptr;

	if (UbaSessionServer == nullptr)
	{
		return;
	}

	NetworkServer_Stop(UbaServer);

	Scheduler_Destroy(UbaScheduler);
	SessionServer_Destroy(UbaSessionServer);
	StorageServer_Destroy(UbaStorageServer);
	NetworkServer_Destroy(UbaServer);

	UbaScheduler = nullptr;
	UbaSessionServer = nullptr;
	UbaStorageServer = nullptr;
	UbaServer = nullptr;
}

uint32 FUbaJobProcessor::Run()
{
	bIsWorkDone = false;
	
	const double CurrentTime = FPlatformTime::Seconds();
	double LastTimeSinceHadJobs = CurrentTime;
	double LastHeartBeat = CurrentTime;
	double LastTraceSnapshot = CurrentTime;

	while (!bForceStop)
	{
		const double ElapsedSeconds = (FPlatformTime::Seconds() - LastTimeSinceHadJobs);

		uint32 NumUbaQueuedJobs = 0;
		uint32 NumActiveLocalJobs = 0;
		uint32 NumActiveRemoteJobs = 0;
		uint32 NumFinishedJobs = 0;
		bool bNewTasks = !ControllerModule.PendingRequestedCompilationTasks.IsEmpty();
		// never empty if we have new tasks
		bool bIsEmpty = !bNewTasks;

		const double HeartBeatElapsedSeconds = (FPlatformTime::Seconds() - LastHeartBeat);

		if (UbaScheduler)
		{
			Scheduler_GetStats(UbaScheduler, NumUbaQueuedJobs, NumActiveLocalJobs, NumActiveRemoteJobs, NumFinishedJobs);
			bIsEmpty &= Scheduler_IsEmpty(UbaScheduler);
		}
		const uint32 NumActiveJobs = NumActiveLocalJobs + NumActiveRemoteJobs;

		// We don't want to hog up Horde resources.
		if (UbaScheduler && bIsEmpty && ElapsedSeconds > UbaJobProcessorOptions::MaxTimeWithoutTasks)
		{
			// If we're optimizing job starting, we only want to shutdown UBA if all the processes have terminated
			ShutDownUba();
		}

		// Check if we have new tasks to process
		if (!bIsEmpty)
		{
			if (!UbaScheduler)
			{
				// We have new tasks. Start processing again
				StartUba();
			}

			LastTimeSinceHadJobs = FPlatformTime::Seconds();
		}

		if (UbaScheduler)
		{
			Scheduler_SetMaxLocalProcessors(UbaScheduler, MaxLocalParallelJobs);

			const int32 MaxRemoteCoresToRequest = FMath::Max(0, (int32)(NumUbaQueuedJobs + NumActiveJobs + ControllerModule.PendingRequestedCompilationCount) - MaxLocalParallelJobs);
			if (HordeAgentManager)
			{
				HordeAgentManager->SetTargetCoreCount(MaxRemoteCoresToRequest);
			}
			
			SessionServer_SetMaxRemoteProcessCount(UbaSessionServer, MaxRemoteCoresToRequest);


			const float TaskProcessTimeSliceSeconds = 3.0f;
			float TaskProcessTimeStart = FPlatformTime::Seconds();

			if (bNewTasks)
			{
				while (true)
				{
					FDistributedBuildTask* Task = nullptr;
					// break out of the task loop if we've been processing tasks for more than TaskProcessTimeSliceSeconds, or the pending tasks queue is empty
					// the former ensures we don't go too long without updating the amount of remote help we request (this can cause pathological behaviour when
					// the queue grows large quickly after being empty or small for a period of time).
					if ((FPlatformTime::Seconds() > (TaskProcessTimeStart + TaskProcessTimeSliceSeconds))
						|| !ControllerModule.PendingRequestedCompilationTasks.Dequeue(Task) 
						|| !Task)
					{
						break;
					}
					ControllerModule.PendingRequestedCompilationCount--;
					RunTaskWithUba(Task);
				}
			}


			UpdateStats();

			if (HeartBeatElapsedSeconds > UbaJobProcessorOptions::HeartBeatInterval)
			{
				// only print heartbeat log messages if currently executing tasks
				UE_LOGF(LogUbaController, Display, "Task Status -- Queued: %u submitted, %u pending -- Active: %u local, %u remote -- Completed: %u", NumUbaQueuedJobs, ControllerModule.PendingRequestedCompilationCount.load(), NumActiveLocalJobs, NumActiveRemoteJobs, NumFinishedJobs);
				LastHeartBeat = FPlatformTime::Seconds();
			}

			// Save snapshot of current trace if periodic saves are enabled
			if (UbaJobProcessorOptions::SaveUbaTraceSnapshotInterval > 0 && UbaSessionServer)
			{
				const int32 SnapshotIntervalElapsedSeconds = (int32)(FPlatformTime::Seconds() - LastTraceSnapshot);
				if (SnapshotIntervalElapsedSeconds >= UbaJobProcessorOptions::SaveUbaTraceSnapshotInterval)
				{
					SaveSnapshotOfTrace();
					LastTraceSnapshot = FPlatformTime::Seconds();
				}
			}
		}
		
		if (ControllerModule.PendingRequestedCompilationTasks.IsEmpty())
		{
			// wait for new tasks if there are none currently available; we only wait on the wake event as long as our shutdown timeout
			// this will be triggered when we have new tasks to process or need to terminate the session (either due to a stop signal 
			// from the controller module, or due to not receiving any new work within the timeout)
			ControllerModule.ProcessorWakeEvent->Wait(UbaJobProcessorOptions::MaxTimeWithoutTasks * 1000);
		}
	}

	ShutDownUba();

	bIsWorkDone = true;
	return 0;
}

void FUbaJobProcessor::Stop()
{
	bForceStop = true;
};

bool FUbaJobProcessor::IsUsingVirtualFiles() const
{
	return UbaJobProcessorOptions::bUseVirtualFiles;
}

void FUbaJobProcessor::StartThread()
{
	Thread = FRunnableThread::Create(this, TEXT("UbaJobProcessor"), 0, TPri_SlightlyBelowNormal, FPlatformAffinity::GetPoolThreadMask());
}

void FUbaJobProcessor::HandleUbaJobFinished(FDistributedBuildTask* CompileTask, TArray<uint8>&& OutputFileData, uint32 ExitCode)
{
	checkf(CompileTask, TEXT("Invalid FDistributedBuildTask processing finished UBA job"));

	const FOutputDataValidator& CustomValidator = CompileTask->CommandData.OutputValidator;
	bool bValidOutput = false;
	int64 FileSize = 0;

	if (CustomValidator)
	{
		// Custom per-task output validation (e.g. PSDB SQLite/compressed format check).
		bValidOutput = CustomValidator(OutputFileData);
	}
	else
	{
		// Default: validate as SCW output format (version uint32 + data-size uint64 header).
		constexpr uint64 VersionAndFileSizeSize = sizeof(uint32) + sizeof(uint64);
		if (OutputFileData.Num() > VersionAndFileSizeSize)
		{
			FMemoryReader OutputFile(OutputFileData);
			int32 OutputVersion;
			OutputFile << OutputVersion; // NOTE (SB): Do not care right now about the version.
			OutputFile << FileSize;
			if ((FileSize > 0) && (OutputFile.TotalSize() >= FileSize))
			{
				bValidOutput = true;
			}
		}
	}

	if (!bValidOutput)
	{
		TStringBuilder<64> ExpectedFileSizeStr;
		if (FileSize > 0)
		{
			ExpectedFileSizeStr << FileSize;
		}
		else
		{
			ExpectedFileSizeStr << TEXT("Unknown");
		}
		UE_LOGF(LogUbaController, Display, "Output virtual file [%ls] size is not correct (task %ls, exit code %d) (Expected Size : [%ls], Actual Size : [%d]); job will retry locally", *CompileTask->CommandData.OutputFileName, *CompileTask->CommandData.Command, ExitCode, ExpectedFileSizeStr.ToString(), OutputFileData.Num());

		// Save current snapshot of UBA trace in case this failure crashes the cook later on
		SaveSnapshotOfTrace();

		// explicitly empty the contents of the output file in the case that we got something non-empty but somehow corrupted.
		// an empty output file contents array (along with the file not existing on disk) will signal the distributed compile thread to retry the job locally.
		OutputFileData.Empty();
	}

	CompileTask->Finalize(0, MoveTemp(OutputFileData));
}

bool FUbaJobProcessor::HasJobsInFlight() const
{
	if (!UbaScheduler)
	{
		return false;
	}

	return !Scheduler_IsEmpty(UbaScheduler);
}

bool FUbaJobProcessor::PollStats(FDistributedBuildStats& OutStats)
{
	// Return current stats and reset internal data
	FScopeLock StatsLockGuard(&StatsLock);
	OutStats = Stats;
	Stats = FDistributedBuildStats();
	return true;
}

void FUbaJobProcessor::UpdateStats()
{
	if (HordeAgentManager)
	{
		FScopeLock StatsLockGuard(&StatsLock);

		// Update maximum
		Stats.MaxRemoteAgents = FMath::Max(Stats.MaxRemoteAgents, (uint32)HordeAgentManager->GetAgentCount());
		Stats.MaxActiveAgentCores = FMath::Max(Stats.MaxActiveAgentCores, HordeAgentManager->GetActiveCoreCount());
	}
}

void FUbaJobProcessor::SaveSnapshotOfTrace()
{
	if (!TraceOutputFilename.IsEmpty())
	{
		UE_LOGF(LogUbaController, Display, "Saving snapshot of UBA trace: %ls", *TraceOutputFilename);
		SessionServer_SaveSnapshotOfTrace(UbaSessionServer);
	}
}
