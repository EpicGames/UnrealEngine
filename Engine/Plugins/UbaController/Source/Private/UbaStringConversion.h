// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/StringConv.h"
#include "UbaHost.h"

using FStringToUbaStringConversion = TStringConversion<TStringConvert<TCHAR, uba::tchar>>;
using FUbaStringToStringConversion = TStringConversion<TStringConvert<uba::tchar, TCHAR>>;

#define TCHAR_TO_UBASTRING(STR) FStringToUbaStringConversion(STR).Get()
#define UBASTRING_TO_TCHAR(STR) FUbaStringToStringConversion(STR).Get()

