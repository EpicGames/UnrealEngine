// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Queue.h"
#include "DistributedBuildControllerInterface.h"

class FUbaJobProcessor;
struct FTaskCommandData;

DECLARE_LOG_CATEGORY_EXTERN(LogUbaController, Log, All);

class FUbaControllerModule : public IDistributedBuildController , public TSharedFromThis<FUbaControllerModule>
{
public:
	
	FUbaControllerModule();
	virtual ~FUbaControllerModule() override;
	
	UBACONTROLLER_API static FUbaControllerModule& Get();
	UBACONTROLLER_API virtual bool IsSupported() override final;

	virtual const FString GetName() override final { return FString("UBA Controller"); };
	
	virtual void StartupModule() override final;
	virtual void ShutdownModule() override final;
	virtual void InitializeController() override final;
	virtual bool SupportsLocalWorkers() override final;
	virtual FString CreateUniqueFilePath() override final;
	virtual bool SupportsVirtualFiles() const override final;
	virtual TFuture<FDistributedBuildTaskResult> EnqueueTask(const FTaskCommandData& CommandData) override final;
	virtual bool PollStats(FDistributedBuildStats& OutStats) override final;

	void CleanWorkingDirectory() const;

	const FString& GetRootWorkingDirectory() const { return RootWorkingDirectory; }
	const FString& GetWorkingDirectory() const { return WorkingDirectory; }
	const FString& GetDebugInfoPath() const { return DebugInfoPath; }

	// Queue of tasks submitted by the engine, but not yet dispatched to the controller.
	TQueue<FDistributedBuildTask*, EQueueMode::Spsc> PendingRequestedCompilationTasks;
	std::atomic<uint32> PendingRequestedCompilationCount;

	// Event used to wake the FUbaJobProcessor instance when new tasks are available or a shutdown is requested (the instance will block on this event when out of work)
	FEventRef ProcessorWakeEvent;

	virtual void SetMaxLocalWorkers(int32 InMaxNumLocalWorkers) override final
	{
		MaxNumLocalWorkers = InMaxNumLocalWorkers;
	}

	inline int32 GetMaxNumLocalWorkers() const
	{
		return MaxNumLocalWorkers;
	}

	static FString GetTempDir();

private:
	void LoadDependencies();

	void StartDispatcherThread();
	void StopAndWaitForDispatcherThread();

	bool bSupported;
	bool bModuleInitialized;
	bool bControllerInitialized;
	
	FString RootWorkingDirectory;
	FString WorkingDirectory;
	FString DebugInfoPath;

	TAtomic<int32> NextFileID;
	TAtomic<int32> NextTaskID;
	TAtomic<int32> MaxNumLocalWorkers = -1;

	TUniquePtr<FUbaJobProcessor> JobDispatcherThread;
};
