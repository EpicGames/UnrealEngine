// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/PlatformAtomics.h"
#include "HAL/Runnable.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UbaHost.h"
#include "DistributedBuildControllerInterface.h"

class FUbaControllerModule;
class FUbaHordeAgentManager;
struct FDistributedBuildTask;

class FUbaJobProcessor : public FRunnable, public TSharedFromThis<FUbaJobProcessor>
{
public:
	FUbaJobProcessor(FUbaControllerModule& InControllerModule);

	virtual ~FUbaJobProcessor() override;
	
	/** Main loop */
	virtual uint32 Run() override;
	
	/** Aborts the main loop as soon as possible */
	virtual void Stop() override;

	/** Indicates whether the job processor instance is using the UBA virtual file mechanism */
	bool IsUsingVirtualFiles() const;
	
	/** Creates the threads and starts the main loop */
	void StartThread();
	void HandleUbaJobFinished(FDistributedBuildTask* CompileTask, TArray<uint8>&& OutputFileData, uint32 ExitCode);

	/** Used to know when this thread has finished the main loop */
	bool IsWorkDone() const { return bIsWorkDone;};

	bool HasJobsInFlight() const;

	bool PollStats(FDistributedBuildStats& OutStats);

private:

	void UpdateMaxLocalParallelJobs();
	void CalculateKnownInputs();
	void RunTaskWithUba(FDistributedBuildTask* Task);

	void StartUba();
	void ShutDownUba();

	void UpdateStats();

	void SaveSnapshotOfTrace();

	/** The runnable thread */
	FRunnableThread* Thread = nullptr;
	
	FUbaControllerModule& ControllerModule;

	int32 MaxLocalParallelJobs = 0;
	
	/** Used to abort the current processing loop */
	TAtomic<bool> bForceStop;
	
	/** Used to abort the current processing loop */
	uint32 LastTimeCheckedForTasks = 0;
	
	/** Set to true when the main loop finishes*/
	bool bIsWorkDone = false;

	FString TraceOutputFilename;

	uba::NetworkServer* UbaServer = nullptr;
	uba::StorageServer* UbaStorageServer = nullptr;
	uba::SessionServer* UbaSessionServer = nullptr;
	uba::Scheduler* UbaScheduler = nullptr;
	TUniquePtr<FUbaHordeAgentManager> HordeAgentManager;

	uint32 KnownInputsCount = 0;
	TArray<uba::tchar> KnownInputsBuffer;

	uba::CallbackLogWriter LogWriter;

	FCriticalSection StatsLock;
	FDistributedBuildStats Stats;
};