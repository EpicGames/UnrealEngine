// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class UbaController : ModuleRules
{
	public UbaController(ReadOnlyTargetRules TargetRules) : base(TargetRules)
	{
		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"DistributedBuildInterface",
			"Projects",
			"RenderCore",
			"TargetPlatform",
			"Horde",
			"HTTP",
			"Sockets",
			"UbaCoordinatorHorde",
			"Json"
		});

		string UbaSourcePath = Path.Combine(EngineDirectory, "Source", "Programs", "UnrealBuildAccelerator");

		PublicIncludePaths.Add(UbaSourcePath);
		PublicIncludePaths.Add(Path.Combine(UbaSourcePath, "Core", "Public"));
		PublicIncludePaths.Add(Path.Combine(UbaSourcePath, "Common", "Public"));
		PublicIncludePaths.Add(Path.Combine(UbaSourcePath, "Host", "Public"));

		string UbaBinariesPath = Path.Combine(EngineDirectory, "Binaries", Target.Platform.ToString(), "UnrealBuildAccelerator");
		
		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			UbaBinariesPath = Path.Combine(UbaBinariesPath, TargetRules.Architecture.WindowsLibDir);
			
			PublicAdditionalLibraries.Add(Path.Combine(UbaBinariesPath, "UbaHost.lib"));
			PrivateRuntimeLibraryPaths.Add(Path.Combine(ModuleDirectory, "lib"));

			PublicDelayLoadDLLs.Add("UbaHost.dll");
			RuntimeDependencies.Add(Path.Combine(UbaBinariesPath, "UbaHost.dll"));
			AdditionalPropertiesForReceipt.Add("Uba", Path.Combine(UbaBinariesPath, "UbaDetours.dll"));
			AdditionalPropertiesForReceipt.Add("Uba", Path.Combine(UbaBinariesPath, "UbaAgent.exe"));

			// todo: this only exists for x64 atm. Is not needed
			//AdditionalPropertiesForReceipt.Add("Uba", Path.Combine(UbaBinariesPath, "UbaWine.dll.so"));

			// Only for windows atm
			AdditionalPropertiesForReceipt.Add("Uba", Path.Combine(UbaBinariesPath, "UbaHost.toml"));
			AdditionalPropertiesForReceipt.Add("Uba", Path.Combine(UbaBinariesPath, "UbaAgent.toml"));
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PublicDelayLoadDLLs.Add(Path.Combine(UbaBinariesPath, "libUbaHost.dylib"));
			RuntimeDependencies.Add(Path.Combine(UbaBinariesPath, "libUbaHost.dylib"));
			AdditionalPropertiesForReceipt.Add("Uba", Path.Combine(UbaBinariesPath, "libUbaDetours.dylib"));
			AdditionalPropertiesForReceipt.Add("Uba", Path.Combine(UbaBinariesPath, "UbaAgent"));
		}
		else if (Target.Platform == UnrealTargetPlatform.Linux)
		{
			PublicDelayLoadDLLs.Add(Path.Combine(UbaBinariesPath, "libUbaHost.so"));
			RuntimeDependencies.Add(Path.Combine(UbaBinariesPath, "libUbaHost.so"));
			AdditionalPropertiesForReceipt.Add("Uba", Path.Combine(UbaBinariesPath, "libUbaDetours.so"));
			AdditionalPropertiesForReceipt.Add("Uba", Path.Combine(UbaBinariesPath, "UbaAgent"));
		}
	}
}
