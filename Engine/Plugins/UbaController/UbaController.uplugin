{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "UBA Controller",
	"Description" : "Adds support for shader compiling distribution using UnrealBuildAccelerator (UBA)",
	"Category" : "Build Distribution",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"IsExperimentalVersion": true,
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,

	"Modules" :
	[
		{
			"Name" : "UbaController",
			"Type" : "Editor",
			"LoadingPhase" : "EarliestPossible",
			"PlatformAllowList": ["Win64", "Mac", "Linux"]
		}
	],

	"SupportedTargetPlatforms": ["Win64", "Mac", "Linux"]
}